{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Motion Design Scene State Integration",
	"Description": "",
	"Category": "Other",
	"CreatedBy": "",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "AvalancheSceneState",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "AvalancheSceneStateBlueprint",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default",
			"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "AvalancheSceneStateEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "Avalanche",
			"Enabled": true
		},
		{
			"Name": "SceneState",
			"Enabled": true
		}
	]
}