// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheSceneState : ModuleRules
{
	public AvalancheSceneState(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"AvalancheRemoteControl",
				"AvalancheSequence",
				"AvalancheTag",
				"Core",
				"CoreUObject",
				"SceneState",
				"SceneStateBinding",
				"SceneStateTasks",
				"SceneStateGameplay",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Avalanche",
				"Engine",
				"PropertyBindingUtils",
				"RemoteControl",
				"RemoteControlLogic",
				"SceneStateEvent",
			}
		);

		if (Target.Type == TargetRules.TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"SceneStateBlueprint",
					"SceneStateBlueprintEditor",
				});
		}
	}
}