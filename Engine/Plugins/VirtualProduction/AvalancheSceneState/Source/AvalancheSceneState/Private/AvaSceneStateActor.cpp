// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSceneStateActor.h"
#include "AvaSceneStateComponent.h"
#include "Engine/World.h"
#include "SceneStateGeneratedClass.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

#if WITH_EDITOR
#include "SceneStateBlueprint.h"
#include "SceneStateBlueprintFactory.h"
#endif

AAvaSceneStateActor::AAvaSceneStateActor(const FObjectInitializer& InObjectInitializer)
	: Super(InObjectInitializer.SetDefaultSubobjectClass<UAvaSceneStateComponent>(ASceneStateActor::SceneStateComponentName))
{
#if WITH_EDITOR
	FWorldDelegates::OnWorldCleanup.AddUObject(this, &AAvaSceneStateActor::OnWorldCleanup);
	FWorldDelegates::OnPostWorldRename.AddUObject(this, &AAvaSceneStateActor::OnWorldRenamed);
#endif
}

#if WITH_EDITOR
FString AAvaSceneStateActor::GetDefaultActorLabel() const
{
	return TEXT("Motion Design Scene State");
}

void AAvaSceneStateActor::PostLoad()
{
	Super::PostLoad();
	bListedInSceneOutliner = true;
	SetSceneStateBlueprint(Cast<USceneStateBlueprint>(SceneStateBlueprint));
}

void AAvaSceneStateActor::PostDuplicate(bool bInDuplicateForPIE)
{
	Super::PostDuplicate(bInDuplicateForPIE);

	if (bInDuplicateForPIE)
	{
		SetSceneStateBlueprint(nullptr);
	}
	else
	{
		SetSceneStateBlueprint(Cast<USceneStateBlueprint>(SceneStateBlueprint));
		UpdateSceneStateClass();
	}
}

void AAvaSceneStateActor::BeginDestroy()
{
	Super::BeginDestroy();
	FWorldDelegates::OnWorldCleanup.RemoveAll(this);
	FWorldDelegates::OnPostWorldRename.RemoveAll(this);
}

void AAvaSceneStateActor::UpdateSceneStateClass()
{
	if (SceneStateBlueprint)
	{
		SetSceneStateClass(Cast<USceneStateGeneratedClass>(SceneStateBlueprint->GeneratedClass));
	}
	else
	{
		SetSceneStateClass(nullptr);
	}
}

void AAvaSceneStateActor::SetSceneStateBlueprint(USceneStateBlueprint* InSceneStateBlueprint)
{
	if (SceneStateBlueprint)
	{
		SceneStateBlueprint->OnCompiled().RemoveAll(this);
	}

	SceneStateBlueprint = InSceneStateBlueprint;

	if (SceneStateBlueprint)
	{
		SceneStateBlueprint->OnCompiled().AddUObject(this, &AAvaSceneStateActor::OnSceneStateRecompiled);
	}
}

void AAvaSceneStateActor::OnSceneStateRecompiled(UBlueprint* InCompiledBlueprint)
{
	ensure(InCompiledBlueprint == SceneStateBlueprint);
	UpdateSceneStateClass();
}

void AAvaSceneStateActor::OnWorldRenamed(UWorld* InNewWorld)
{
	// Rename Blueprints to the same Outer/Name so that the latest information is propagated to renaming the generated class
	// Not doing this can result in the generated class properties to be bound to the old name
	if (SceneStateBlueprint)
	{
		SceneStateBlueprint->Rename(*SceneStateBlueprint->GetName(), SceneStateBlueprint->GetOuter(), REN_DontCreateRedirectors);
	}
}

void AAvaSceneStateActor::OnWorldCleanup(UWorld* InWorld, bool bInSessionEnded, bool bInCleanupResources)
{
	if (!bInCleanupResources || !SceneStateBlueprint)
	{
		return;
	}

	// Ignore cleanups from other worlds
	if (GetTypedOuter<UWorld>() != InWorld)
	{
		return;
	}

	CleanupSceneState();
}

void AAvaSceneStateActor::CleanupSceneState()
{
	if (SceneStateBlueprint)
	{
		const FName ObjectName = MakeUniqueObjectName(GetTransientPackage()
			, SceneStateBlueprint->GetClass()
			, FName(*FString::Printf(TEXT("%s_Trashed"), *SceneStateBlueprint->GetName())));

		SceneStateBlueprint->Rename(*ObjectName.ToString()
			, GetTransientPackage()
			, REN_DontCreateRedirectors | REN_NonTransactional | REN_DoNotDirty);
	}

	SetSceneStateBlueprint(nullptr);
	SetSceneStateClass(nullptr);
}
#endif
