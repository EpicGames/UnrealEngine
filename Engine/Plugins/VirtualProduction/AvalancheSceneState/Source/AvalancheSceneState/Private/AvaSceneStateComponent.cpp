// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSceneStateComponent.h"
#include "AvaSceneStatePlayer.h"

UAvaSceneStateComponent::UAvaSceneStateComponent(const FObjectInitializer& InObjectInitializer)
	: Super(InObjectInitializer.SetDefaultSubobjectClass<UAvaSceneStatePlayer>(UAvaSceneStateComponent::SceneStatePlayerName))
{
}
