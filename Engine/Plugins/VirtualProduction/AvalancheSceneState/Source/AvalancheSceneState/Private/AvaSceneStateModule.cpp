// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSceneStateLog.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogAvaSceneState);

IMPLEMENT_MODULE(FDefaultModuleImpl, AvalancheSceneState)
