// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSceneStatePlayer.h"

UAvaSceneStatePlayer::UAvaSceneStatePlayer()
{
#if WITH_EDITOR
	bEditableSceneStateClass = false;
#endif
}
