// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSceneStateSchema.h"

#if WITH_EDITOR
#include "SceneStateUtils.h"
#include "Tasks/SceneStateTask.h"
#include "Tasks/SceneStateTaskMetadata.h"
#endif

#if WITH_EDITOR
bool UAvaSceneStateSchema::OnIsTaskStructAllowed(TSubScriptStructOf<FSceneStateTask> InTaskStruct) const
{
	// Allow Motion Design and Core/Utility tasks
	constexpr bool bIncludeSuperStructs = false;
	return UE::SceneState::Editor::StructHasAnyMetaData(InTaskStruct, 
		{
			UE::SceneState::Editor::MD_CoreTask,
			UE::SceneState::Editor::MD_UtilityTask,
			UE::AvaSceneState::Editor::MD_MotionDesignTask,
		}, bIncludeSuperStructs);
}
#endif
