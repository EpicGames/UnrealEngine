// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSceneStateUtils.h"
#include "AvaSceneStateLog.h"
#include "AvaSceneSubsystem.h"
#include "Engine/Level.h"
#include "SceneStateExecutionContext.h"

namespace UE::AvaSceneState
{

IAvaSceneInterface* FindSceneInterface(const FSceneStateExecutionContext& InContext)
{
	UObject* ContextObject = InContext.GetContextObject();
	if (!ContextObject)
	{
		UE_LOGF(LogAvaSceneState, Warning, "[%ls] Scene State could not find a valid Context Object", *InContext.GetExecutionContextName());
		return nullptr;
	}

	ULevel* ContextLevel = Cast<ULevel>(ContextObject);
	if (!ContextLevel)
	{
		ContextLevel = ContextObject->GetTypedOuter<ULevel>();
	}

	IAvaSceneInterface* SceneInterface = UAvaSceneSubsystem::FindSceneInterface(ContextLevel);
	if (!SceneInterface)
	{
		UE_LOGF(LogAvaSceneState, Warning, "[%ls] Failed to find Motion Design Scene Interface!", *InContext.GetExecutionContextName());
		return nullptr;
	}

	return SceneInterface;
}

} // UE::AvaSceneState
