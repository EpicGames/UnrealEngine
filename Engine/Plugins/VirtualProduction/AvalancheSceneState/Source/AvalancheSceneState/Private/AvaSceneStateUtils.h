// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class IAvaSceneInterface;
struct FSceneStateExecutionContext;

namespace UE::AvaSceneState
{
	IAvaSceneInterface* FindSceneInterface(const FSceneStateExecutionContext& InContext);
}
