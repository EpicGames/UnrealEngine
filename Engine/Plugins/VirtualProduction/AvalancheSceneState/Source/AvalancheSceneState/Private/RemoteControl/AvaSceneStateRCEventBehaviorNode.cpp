// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControl/AvaSceneStateRCEventBehaviorNode.h"
#include "AvaSceneStateLog.h"
#include "Engine/World.h"
#include "RemoteControl/AvaSceneStateRCEventBehavior.h"
#include "SceneStateEventUtils.h"
#include "StructUtils/UserDefinedStruct.h"

#define LOCTEXT_NAMESPACE "AvaSceneStateRCEventBehaviorNode"

UAvaSceneStateRCEventBehaviorNode::UAvaSceneStateRCEventBehaviorNode()
{
	DisplayName = LOCTEXT("DisplayName", "Broadcast Event");
	BehaviorDescription = LOCTEXT("Description", "Broadcasts a defined event after applying the actions");
}

bool UAvaSceneStateRCEventBehaviorNode::Execute(URCBehaviour* InBehavior) const
{
	return true;
}

bool UAvaSceneStateRCEventBehaviorNode::IsSupported(URCBehaviour* InBehavior) const
{
	return InBehavior
		&& InBehavior->IsA<UAvaSceneStateRCEventBehavior>()
		&& UE::SceneState::GetContextWorld(InBehavior);
}

void UAvaSceneStateRCEventBehaviorNode::OnPassed(URCBehaviour* InBehavior) const
{
	if (const UWorld* World = UE::SceneState::GetContextWorld(InBehavior))
	{
		UE::SceneState::BroadcastEvent(World, Event);
	}
	else
	{
		UE_LOGF(LogAvaSceneState, Error, "BroadcastEvent failed. Behavior node %ls could not find a valid world.", *GetFullName())
	}
}

UClass* UAvaSceneStateRCEventBehaviorNode::GetBehaviourClass() const
{
	return UAvaSceneStateRCEventBehavior::StaticClass();
}

#undef LOCTEXT_NAMESPACE
