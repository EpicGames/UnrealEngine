// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControl/AvaSceneStateRCTask.h"
#include "AvaSceneStateLog.h"
#include "AvaSceneStateSchema.h"
#include "AvaSceneStateUtils.h"
#include "Controller/RCController.h"
#include "IAvaSceneInterface.h"
#include "RCVirtualPropertyContainer.h"
#include "RemoteControlPreset.h"
#include "SceneStateBindingDesc.h"
#include "SceneStateBindingUtils.h"
#include "SceneStateExecutionContext.h"
#include "UObject/PropertyAccessUtil.h"

namespace UE::AvaSceneState::Private
{
	struct FCopyInfo
	{
		const FProperty* SourceProperty;
		const uint8* SourceMemory;
		FProperty* TargetProperty;
		uint8* TargetMemory;
	};

	/**
	 * Tries promoting a given source property value to match the target property type and copies it to the target memory
	 * Derived from FPropertyBindingBindingCollection::PerformCopy
	 * Todo: merge with FPromotionCopy in Motion Design Data Link Integration
	 */
	template<typename InSourcePropertyType>
	class FPromotionCopy
	{
		template<typename InTargetPropertyType>
		bool CopySingle(const FCopyInfo& InCopyInfo, const InSourcePropertyType& InSourceProperty)
		{
			using FSourceCppType = InSourcePropertyType::TCppType;
			using FTargetCppType = InTargetPropertyType::TCppType;

			static_assert(std::is_convertible_v<FSourceCppType, FTargetCppType>);

			if (InTargetPropertyType* TargetProperty = CastField<InTargetPropertyType>(InCopyInfo.TargetProperty))
			{
				TargetProperty->SetPropertyValue(InCopyInfo.TargetMemory, InSourceProperty.GetPropertyValue(InCopyInfo.SourceMemory));
				return true;
			}
			return false;
		}

	public:
		/** Tries promoting from a source property type to any of the given target property types */
		template<typename ...InTargetPropertyTypes>
		bool Copy(const FCopyInfo& InCopyInfo)
		{
			if (const InSourcePropertyType* SourceProperty = CastField<InSourcePropertyType>(InCopyInfo.SourceProperty))
			{
				return (this->CopySingle<InTargetPropertyTypes>(InCopyInfo, *SourceProperty) || ...);
			}
			return false;
		}
	};

	bool PromoteCopy(const FCopyInfo& InCopyInfo)
	{
		// Bool Promotions
		return FPromotionCopy<FBoolProperty>().Copy<FByteProperty, FIntProperty, FUInt32Property, FInt64Property, FFloatProperty, FDoubleProperty>(InCopyInfo)
			// Byte Promotion
			|| FPromotionCopy<FByteProperty>().Copy<FIntProperty, FUInt32Property, FInt64Property, FFloatProperty, FDoubleProperty>(InCopyInfo)
			// Int32 Promotions
			|| FPromotionCopy<FIntProperty>().Copy<FInt64Property, FFloatProperty, FDoubleProperty>(InCopyInfo)
			// Uint32 Promotions
			|| FPromotionCopy<FUInt32Property>().Copy<FInt64Property, FFloatProperty, FDoubleProperty>(InCopyInfo)
			// Float Promotions
			|| FPromotionCopy<FFloatProperty>().Copy<FIntProperty, FInt64Property, FDoubleProperty>(InCopyInfo)
			// Double Promotions
			|| FPromotionCopy<FDoubleProperty>().Copy<FIntProperty, FInt64Property, FFloatProperty>(InCopyInfo)
		;
	}

} // UE::AvaSceneState::Private

FAvaSceneStateRCTask::FAvaSceneStateRCTask()
{
	SetFlags(ESceneStateTaskFlags::HasBindingExtension);
}

#if WITH_EDITOR
const UScriptStruct* FAvaSceneStateRCTask::OnGetTaskInstanceType() const
{
	return FInstanceDataType::StaticStruct();
}

void FAvaSceneStateRCTask::OnBuildTaskInstance(UObject* InOuter, FStructView InTaskInstance) const
{
	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();

	const FGuid OldControllerValuesId = Instance.ControllerValuesId;
	Instance.ControllerValuesId = FGuid::NewGuid();

	UE::SceneState::HandleStructIdChanged(*InOuter, OldControllerValuesId, Instance.ControllerValuesId);
}
#endif

const FSceneStateTaskBindingExtension* FAvaSceneStateRCTask::OnGetBindingExtension() const
{
	return &Binding;
}

void FAvaSceneStateRCTask::OnStart(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const
{
	ON_SCOPE_EXIT
	{
		Finish(InContext, InTaskInstance);
	};

	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();

	const UPropertyBag* SourcePropertyBag = Instance.ControllerValues.GetPropertyBagStruct();
	const uint8* SourceValuesMemory = Instance.ControllerValues.GetValue().GetMemory();
	if (!SourcePropertyBag)
	{
		UE_LOGF(LogAvaSceneState, Warning, "[%ls] No valid source data to copy from!", *InContext.GetExecutionContextName());
		return;
	}

	URemoteControlPreset* Preset;
	FStructView TargetControllerDataView;
	if (!GetControllerDataView(InContext, TargetControllerDataView, Preset))
	{
		return;
	}

	// At most, each entry in the source property bag will copy to a controller
	TArray<URCController*> ModifiedControllers;
	ModifiedControllers.Reserve(Instance.ControllerValues.GetNumPropertiesInBag());

	for (const FPropertyBagPropertyDesc& SourcePropertyDesc : SourcePropertyBag->GetPropertyDescs())
	{
		if (!SourcePropertyDesc.CachedProperty)
		{
			UE_LOGF(LogAvaSceneState, Warning, "[%ls] Property '%ls' was not invalid in the controller values!"
				, *InContext.GetExecutionContextName()
				, *SourcePropertyDesc.Name.ToString());
			continue;
		}

		const FAvaSceneStateRCControllerMapping* ControllerMapping = Instance.ControllerMappings.FindByKey(SourcePropertyDesc.ID);
		if (!ControllerMapping)
		{
			UE_LOGF(LogAvaSceneState, Warning, "[%ls] Property '%ls' was not found in the controller mappings!"
				, *InContext.GetExecutionContextName()
				, *SourcePropertyDesc.Name.ToString());
			continue;
		}

		URCController* const Controller = Cast<URCController>(ControllerMapping->TargetController.FindController(Preset));
		if (!Controller)
		{
			UE_LOGF(LogAvaSceneState, Warning, "[%ls] Controller '%ls' was not found in preset '%ls'!"
				, *InContext.GetExecutionContextName()
				, *SourcePropertyDesc.Name.ToString()
				, *Preset->GetName());
			continue;
		}

		FProperty* const TargetProperty = Controller->GetProperty();
		if (!TargetProperty)
		{
			UE_LOGF(LogAvaSceneState, Warning, "[%ls] Controller '%ls' has invalid target property in preset '%ls'"
				, *InContext.GetExecutionContextName()
				, *Controller->DisplayName.ToString()
				, *Preset->GetName());
			continue;
		}

		const UE::PropertyBinding::EPropertyCompatibility Compatibility = UE::PropertyBinding::GetPropertyCompatibility(SourcePropertyDesc.CachedProperty, TargetProperty);
		if (Compatibility == UE::PropertyBinding::EPropertyCompatibility::Incompatible)
		{
			UE_LOGF(LogAvaSceneState, Warning, "[%ls] Controller '%ls' has type incompatibility in preset '%ls'"
				, *InContext.GetExecutionContextName()
				, *Controller->DisplayName.ToString()
				, *Preset->GetName());
			continue;
		}

		const uint8* SourceMemory = SourcePropertyDesc.CachedProperty->ContainerPtrToValuePtr<uint8>(SourceValuesMemory);
		uint8* TargetMemory = TargetProperty->ContainerPtrToValuePtr<uint8>(TargetControllerDataView.GetMemory());

		switch (Compatibility)
		{
		case UE::PropertyBinding::EPropertyCompatibility::Compatible:
			TargetProperty->CopyCompleteValue(TargetMemory, SourceMemory);
			ModifiedControllers.AddUnique(Controller);
			break;

		case UE::PropertyBinding::EPropertyCompatibility::Promotable:
			ensureMsgf(UE::AvaSceneState::Private::PromoteCopy({
				.SourceProperty = SourcePropertyDesc.CachedProperty,
				.SourceMemory = SourceMemory,
				.TargetProperty = TargetProperty,
				.TargetMemory = TargetMemory
			}), TEXT("Promotion failed even though compatibility was deemed as 'promotable'."));
			ModifiedControllers.AddUnique(Controller);
			break;
		}
	}

	TSet<FGuid> ModifiedControllerIds;
	ModifiedControllerIds.Reserve(ModifiedControllers.Num());

	for (URCController* Controller : ModifiedControllers)
	{
		Controller->OnModifyPropertyValue();
		ModifiedControllerIds.Add(Controller->Id);
	}

	Preset->OnControllerModified().Broadcast(Preset, ModifiedControllerIds);
}

bool FAvaSceneStateRCTask::GetControllerDataView(const FSceneStateExecutionContext& InContext, FStructView& OutControllerView, URemoteControlPreset*& OutPreset) const
{
	const IAvaSceneInterface* const SceneInterface = UE::AvaSceneState::FindSceneInterface(InContext);
	if (!SceneInterface)
	{
		return false;
	}

	OutPreset = SceneInterface->GetRemoteControlPreset();
	if (!OutPreset)
	{
		UE_LOGF(LogAvaSceneState, Warning, "[%ls] Failed to find Remote Control Preset in Scene!", *InContext.GetExecutionContextName());
		return false;
	}

	URCVirtualPropertyContainerBase* const ControllerContainer = OutPreset->GetControllerContainer();
	if (!ControllerContainer)
	{
		UE_LOGF(LogAvaSceneState, Warning, "[%ls] Failed to find Remote Control Preset in Scene!", *InContext.GetExecutionContextName());
		return false;
	}

	OutControllerView = ControllerContainer->GetPropertyBagMutableValue();
	return OutControllerView.IsValid();
}
