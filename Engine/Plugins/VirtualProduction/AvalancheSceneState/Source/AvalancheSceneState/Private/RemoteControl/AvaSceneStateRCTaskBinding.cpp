// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControl/AvaSceneStateRCTaskBinding.h"
#include "RemoteControl/AvaSceneStateRCTask.h"

#if WITH_EDITOR
void FAvaSceneStateRCTaskBinding::VisitBindingDescs(FStructView InTaskInstance, TFunctionRef<void(const UE::SceneState::FTaskBindingDesc&)> InFunctor) const
{
	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();

	if (Instance.ControllerValues.IsValid())
	{
		UE::SceneState::FTaskBindingDesc BindingDesc;

		BindingDesc.Id = Instance.ControllerValuesId;
		BindingDesc.Name = TEXT("ControllerValues");
		BindingDesc.DataView = Instance.ControllerValues.GetMutableValue();
		BindingDesc.DataIndex = CONTROLLER_VALUES_DATA_INDEX;

		InFunctor(BindingDesc);
	}
}

void FAvaSceneStateRCTaskBinding::SetBindingBatch(uint16 InDataIndex, uint16 InBatchIndex)
{
	if (InDataIndex == CONTROLLER_VALUES_DATA_INDEX)
	{
		ControllerValuesBatchIndex = InBatchIndex;
	}
}

bool FAvaSceneStateRCTaskBinding::FindDataById(FStructView InTaskInstance, const FGuid& InStructId, FStructView& OutDataView, uint16& OutDataIndex) const
{
	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();
	if (Instance.ControllerValuesId == InStructId && Instance.ControllerValues.IsValid())
	{
		OutDataView = Instance.ControllerValues.GetMutableValue();
		OutDataIndex = CONTROLLER_VALUES_DATA_INDEX;
		return true;
	}
	return false;
}
#endif

bool FAvaSceneStateRCTaskBinding::FindDataByIndex(FStructView InTaskInstance, uint16 InDataIndex, FStructView& OutDataView) const
{
	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();
	if (InDataIndex == CONTROLLER_VALUES_DATA_INDEX && Instance.ControllerValues.IsValid())
	{
		OutDataView = Instance.ControllerValues.GetMutableValue();
		return true;
	}
	return false;
}

void FAvaSceneStateRCTaskBinding::VisitBindingBatches(FStructView InTaskInstance, TFunctionRef<void(uint16, FStructView)> InFunctor) const
{
	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();

	if (Instance.ControllerValues.IsValid())
	{
		InFunctor(ControllerValuesBatchIndex, Instance.ControllerValues.GetMutableValue());
	}
}
