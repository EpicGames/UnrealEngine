// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sequence/AvaSceneStatePlaySequenceTask.h"
#include "AvaSceneStateLog.h"
#include "AvaSceneStateSchema.h"
#include "AvaSceneStateUtils.h"
#include "AvaSequence.h"
#include "AvaSequencePlayer.h"
#include "AvaSequenceSubsystem.h"
#include "IAvaSceneInterface.h"
#include "SceneStateExecutionContext.h"
#include "Tasks/SceneStateTaskExecutionContext.h"

#if WITH_EDITOR
const UScriptStruct* FAvaSceneStatePlaySequenceTask::OnGetTaskInstanceType() const
{
	return FInstanceDataType::StaticStruct();
}
#endif

void FAvaSceneStatePlaySequenceTask::OnStart(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const
{
	const IAvaSceneInterface* const SceneInterface = UE::AvaSceneState::FindSceneInterface(InContext);
	if (!SceneInterface)
	{
		Finish(InContext, InTaskInstance);
		return;
	}

	IAvaSequencePlaybackObject* const PlaybackObject = SceneInterface->GetPlaybackObject();
	if (!PlaybackObject)
	{
		UE_LOGF(LogAvaSceneState, Warning, "[%ls] Failed to find Playback Object in Scene Interface!", *InContext.GetExecutionContextName());
		Finish(InContext, InTaskInstance);
		return;
	}

	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();

	Instance.OnSequenceFinishedHandle = UAvaSequencePlayer::OnSequenceFinished()
		.AddStatic(&FAvaSceneStatePlaySequenceTask::OnSequenceStopped, UE::SceneState::FTaskExecutionContext(*this, InContext));

	switch (Instance.SequenceQueryType)
	{
	case EAvaTransitionSequenceQueryType::Name:
		Instance.SequencePlayers = PlaybackObject->PlaySequencesByLabel(Instance.SequenceName, Instance.PlaySettings);
		break;

	case EAvaTransitionSequenceQueryType::Tag:
		Instance.SequencePlayers = PlaybackObject->PlaySequencesByTag(Instance.SequenceTag, /*bPerformExactMatch*/true, Instance.PlaySettings);
		break;
	}

	if (Instance.SequencePlayers.IsEmpty() || Instance.WaitType == EAvaTransitionSequenceWaitType::NoWait)
	{
		Finish(InContext, InTaskInstance);
	}
}

void FAvaSceneStatePlaySequenceTask::OnStop(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance, ESceneStateTaskStopReason InStopReason) const
{
	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();

	UAvaSequencePlayer::OnSequenceFinished().Remove(Instance.OnSequenceFinishedHandle);
	Instance.OnSequenceFinishedHandle.Reset();
}

void FAvaSceneStatePlaySequenceTask::OnSequenceStopped(UAvaSequencePlayer* InPlayer, UAvaSequence* InSequence, UE::SceneState::FTaskExecutionContext InTaskContext)
{
	FInstanceDataType* Instance = InTaskContext.GetTaskInstance().GetPtr<FInstanceDataType>();
	if (!Instance)
	{
		InTaskContext.FinishTask();
		return;
	}

	Instance->SequencePlayers.Remove(InPlayer);
	if (Instance->SequencePlayers.IsEmpty())
	{
		InTaskContext.FinishTask();
	}
}
