// Copyright Epic Games, Inc. All Rights Reserved.

#include "Ticker/AvaSceneStateSpawnerTickerTask.h"
#include "AvaSceneStateSchema.h"
#include "Framework/Ticker/AvaTickerComponent.h"
#include "GameFramework/Actor.h"

#define LOCTEXT_NAMESPACE "AvaSceneStateSpawnerTickerTask"

#if WITH_EDITOR
const UScriptStruct* FAvaSceneStateSpawnerTickerTask::OnGetTaskInstanceType() const
{
	return FInstanceDataType::StaticStruct();
}
#endif

bool FAvaSceneStateSpawnerTickerTask::ShouldSpawnActor(FStructView InTaskInstance, FText& OutErrorMessage) const
{
	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();
	if (!Instance.Ticker)
	{
		OutErrorMessage = LOCTEXT("Error_InvalidTickerActor", "Specified ticker actor is invalid.");
		return false;
	}

	if (!Instance.Ticker->FindComponentByClass<UAvaTickerComponent>())
	{
		OutErrorMessage = LOCTEXT("Error_NoTickerComponent", "Specified actor does not contain a ticker component.");
		return false;
	}

	return true;
}

void FAvaSceneStateSpawnerTickerTask::OnActorSpawned(AActor* InActorChecked, FStructView InTaskInstance) const
{
	FInstanceDataType& Instance = InTaskInstance.Get<FInstanceDataType>();

	// Ensure here since ShouldSpawnActor should've already validated that the actor is valid
	if (!ensureMsgf(Instance.Ticker, TEXT("Valid ticker actor unexpectedly not found in task instance")))
	{
		return;
	}

	UAvaTickerComponent* TickerComponent = Instance.Ticker->FindComponentByClass<UAvaTickerComponent>();

	// Ensure here since ShouldSpawnActor should've already validated that the actor has a ticker component
	if (!ensureMsgf(TickerComponent, TEXT("Ticker actor %s expected to have a ticker component"), *Instance.Ticker->GetActorNameOrLabel()))
	{
		return;
	}

	TickerComponent->QueueActor(InActorChecked);
}

#undef LOCTEXT_NAMESPACE
