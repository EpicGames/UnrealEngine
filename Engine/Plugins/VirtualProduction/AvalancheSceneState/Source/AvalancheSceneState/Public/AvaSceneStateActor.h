// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "SceneStateActor.h"
#include "AvaSceneStateActor.generated.h"

#define UE_API AVALANCHESCENESTATE_API

#if WITH_EDITOR
class UBlueprint;
class USceneStateBlueprint;
#endif

/** Scene State Actor that has additional editor-only data holding the Scene State Blueprint */
UCLASS(MinimalAPI, NotBlueprintable, NotPlaceable, DisplayName="Motion Design Scene State Actor", HideCategories=(Actor, Collision, Cooking, DataLayers, HLOD, Input, LevelInstance, Networking, Physics, Rendering, Replication, WorldPartition))
class AAvaSceneStateActor : public ASceneStateActor
{
	GENERATED_BODY()

public:
	AAvaSceneStateActor(const FObjectInitializer& InObjectInitializer);

protected:
#if WITH_EDITOR
	//~ Begin AActor
	virtual FString GetDefaultActorLabel() const override;
	virtual bool IsUserManaged() const override { return false; }
	virtual bool SupportsExternalPackaging() const override { return false; }
	//~ End AActor

	//~ Begin UObject
	virtual void PostLoad() override;
	virtual void PostDuplicate(bool bInDuplicateForPIE) override;
	virtual void BeginDestroy() override;
	//~ End UObject

	/** Updates the scene state class to the latest generated class from the scene state blueprint if valid */
	UE_API void UpdateSceneStateClass();

	UE_API void SetSceneStateBlueprint(USceneStateBlueprint* InSceneStateBlueprint);

	/** Called when the scene state blueprint is compiled */
	void OnSceneStateRecompiled(UBlueprint* InCompiledBlueprint);

	/** Called when the world is renamed */
	void OnWorldRenamed(UWorld* InNewWorld);

	/** Called on World destruction / change to clean up scene state blueprint resources */
	void OnWorldCleanup(UWorld* InWorld, bool bInSessionEnded, bool bInCleanupResources);
#endif

private:
#if WITH_EDITOR
	/** Cleans up scene state resources */
	UE_API void CleanupSceneState();
#endif

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UBlueprint> SceneStateBlueprint;
#endif

	friend class FAvaSceneStateExtension;
};

#undef UE_API
