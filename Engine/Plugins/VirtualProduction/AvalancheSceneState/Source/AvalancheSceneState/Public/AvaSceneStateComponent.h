// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateComponent.h"
#include "AvaSceneStateComponent.generated.h"

UCLASS(HideCategories=(Activation, AssetUserData, Cooking, Input, Navigation, Tags))
class UAvaSceneStateComponent : public USceneStateComponent
{
	GENERATED_BODY()

	UAvaSceneStateComponent(const FObjectInitializer& InObjectInitializer);
};
