// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateComponentPlayer.h"
#include "AvaSceneStatePlayer.generated.h"

UCLASS()
class UAvaSceneStatePlayer : public USceneStateComponentPlayer
{
	GENERATED_BODY()

	UAvaSceneStatePlayer();
};
