// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateGameplaySchema.h"
#include "AvaSceneStateSchema.generated.h"

#define UE_API AVALANCHESCENESTATE_API

#if WITH_EDITOR
namespace UE::AvaSceneState::Editor
{
	/** 
	 * Metadata to identify a task as a Motion Design task 
	 * Derived tasks will NOT be recognized as 'MotionDesignTask' unless they also specify this metadata.
	 */
	const FLazyName MD_MotionDesignTask(TEXT("MotionDesignTask"));
}
#endif

/** Schema for Motion Design scene states. */
UCLASS(MinimalAPI, DisplayName="Motion Design Scene State Schema")
class UAvaSceneStateSchema : public USceneStateGameplaySchema
{
	GENERATED_BODY()

protected:
	//~ Begin USceneStateSchema
#if WITH_EDITOR
	UE_API virtual bool OnIsTaskStructAllowed(TSubScriptStructOf<FSceneStateTask> InTaskStruct) const override;
#endif
	//~ End USceneStateSchema
};

#undef UE_API
