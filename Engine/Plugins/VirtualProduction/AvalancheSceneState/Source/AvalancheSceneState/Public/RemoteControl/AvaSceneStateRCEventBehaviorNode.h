// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Behaviour/RCBehaviourNode.h"
#include "SceneStateEventTemplate.h"
#include "AvaSceneStateRCEventBehaviorNode.generated.h"

UCLASS(MinimalAPI)
class UAvaSceneStateRCEventBehaviorNode : public URCBehaviourNode
{
	GENERATED_BODY()

public:
	UAvaSceneStateRCEventBehaviorNode();

protected:
	//~ Begin URCBehaviourNode
	bool Execute(URCBehaviour* InBehavior) const override;
	bool IsSupported(URCBehaviour* InBehavior) const override;
	void OnPassed(URCBehaviour* InBehavior) const;
	UClass* GetBehaviourClass() const override;
	//~ End URCBehaviourNode

private:
	UPROPERTY(EditAnywhere, Category="Scene State")
	FSceneStateEventTemplate Event;
};
