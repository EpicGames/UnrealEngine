// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaRCControllerId.h"
#include "AvaSceneStateRCTaskBinding.h"
#include "Misc/Guid.h"
#include "StructUtils/PropertyBag.h"
#include "Tasks/SceneStateTask.h"
#include "Tasks/SceneStateTaskBindingExtension.h"
#include "Tasks/SceneStateTaskInstance.h"
#include "AvaSceneStateRCTask.generated.h"

class URemoteControlPreset;

USTRUCT()
struct FAvaSceneStateRCControllerMapping
{
	GENERATED_BODY()

	bool operator==(const FGuid& InPropertyId) const
	{
		return SourcePropertyId == InPropertyId;
	}

	/** Id to the controller to copy the value to */
	UPROPERTY(EditAnywhere, Category="Scene State")
	FAvaRCControllerId TargetController;

	/** Id of the property (in property bag) whose value will be copied into the target controller */
	UPROPERTY(EditAnywhere, Category="Scene State")
	FGuid SourcePropertyId;
};

/**
 * Instance Data for the RC Task
 * @see FAvaSceneStateRCTask
 */
USTRUCT()
struct FAvaSceneStateRCTaskInstance : public FSceneStateTaskInstance
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	/** Struct Id uniquely identifying ControllerValues */
	UPROPERTY(VisibleAnywhere, Category="Scene State", meta=(NoBinding))
	FGuid ControllerValuesId;
#endif

	UPROPERTY(EditAnywhere, Category="Scene State")
	TArray<FAvaSceneStateRCControllerMapping> ControllerMappings;

	UPROPERTY(EditAnywhere, Category="Scene State")
	FInstancedPropertyBag ControllerValues;
};

USTRUCT(DisplayName="Set RC Controller Values", Category="Motion Design", meta=(MotionDesignTask, RequiresContextWorld))
struct FAvaSceneStateRCTask : public FSceneStateTask
{
	GENERATED_BODY()

	using FInstanceDataType = FAvaSceneStateRCTaskInstance;

	FAvaSceneStateRCTask();

protected:
	//~ Begin FSceneStateTask
#if WITH_EDITOR
	virtual const UScriptStruct* OnGetTaskInstanceType() const override;
	virtual void OnBuildTaskInstance(UObject* InOuter, FStructView InTaskInstance) const override;
#endif
	virtual const FSceneStateTaskBindingExtension* OnGetBindingExtension() const override;
	virtual void OnStart(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const override;
	//~ End FSceneStateTask

private:
	bool GetControllerDataView(const FSceneStateExecutionContext& InContext, FStructView& OutControllerView, URemoteControlPreset*& OutPreset) const;

	UPROPERTY()
	FAvaSceneStateRCTaskBinding Binding;
};
