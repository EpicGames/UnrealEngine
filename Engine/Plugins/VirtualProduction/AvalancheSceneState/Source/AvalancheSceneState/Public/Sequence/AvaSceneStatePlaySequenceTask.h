// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AvaSequenceName.h"
#include "AvaSequenceShared.h"
#include "AvaTagHandle.h"
#include "Tasks/SceneStateTask.h"
#include "Tasks/SceneStateTaskInstance.h"
#include "Transition/AvaTransitionSequenceEnums.h"
#include "AvaSceneStatePlaySequenceTask.generated.h"

class IAvaSequencePlaybackObject;
class UAvaSequencePlayer;

namespace UE::SceneState
{
	struct FTaskExecutionContext;
}

USTRUCT()
struct FAvaSceneStatePlaySequenceTaskInstance : public FSceneStateTaskInstance
{
	GENERATED_BODY()

	/** The method to find the sequence to play */
	UPROPERTY(EditAnywhere, Category="Sequence")
	EAvaTransitionSequenceQueryType SequenceQueryType = EAvaTransitionSequenceQueryType::Name;

	/** The name of the sequences to play (if query type is set to name) */
	UPROPERTY(EditAnywhere, Category="Sequence", meta=(EditCondition="SequenceQueryType==EAvaTransitionSequenceQueryType::Name", EditConditionHides))
	FAvaSequenceName SequenceName;

	/** The tag of the sequences to play (if query type is set to tag) */
	UPROPERTY(EditAnywhere, Category="Sequence", meta=(EditCondition="SequenceQueryType==EAvaTransitionSequenceQueryType::Tag", EditConditionHides))
	FAvaTagHandle SequenceTag;

	UPROPERTY(EditAnywhere, Category="Sequence")
	FAvaSequencePlayParams PlaySettings;

	/** The wait type before this task completes */
	UPROPERTY(EditAnywhere, Category="Sequencer")
	EAvaTransitionSequenceWaitType WaitType = EAvaTransitionSequenceWaitType::WaitUntilStop;

	/** Active sequence players on this instance */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UAvaSequencePlayer>> SequencePlayers;

	FDelegateHandle OnSequenceFinishedHandle;
};

USTRUCT(DisplayName="Play Sequence", Category="Motion Design", meta=(MotionDesignTask, RequiresContextWorld, ToolTip="Plays a Motion Design Sequence"))
struct FAvaSceneStatePlaySequenceTask : public FSceneStateTask
{
	GENERATED_BODY()

	using FInstanceDataType = FAvaSceneStatePlaySequenceTaskInstance;

protected:
	//~ Begin FSceneStateTask
#if WITH_EDITOR
	virtual const UScriptStruct* OnGetTaskInstanceType() const override;
#endif
	virtual void OnStart(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance) const override;
	virtual void OnStop(const FSceneStateExecutionContext& InContext, FStructView InTaskInstance, ESceneStateTaskStopReason InStopReason) const override;
	//~ End FSceneStateTask

private:
	static void OnSequenceStopped(UAvaSequencePlayer* InPlayer, UAvaSequence* InSequence, UE::SceneState::FTaskExecutionContext InTaskContext);
};
