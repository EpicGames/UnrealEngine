// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateSpawnActorTask.h"
#include "AvaSceneStateSpawnerTickerTask.generated.h"

#define UE_API AVALANCHESCENESTATE_API

USTRUCT()
struct FAvaSceneStateTickerTaskInstance : public FSceneStateSpawnActorTaskInstance
{
	GENERATED_BODY()

	/** The actor to find the ticker component for */
	UPROPERTY(EditAnywhere, Category="Ticker")
	TObjectPtr<AActor> Ticker;
};

/**
 * Spawns an actor and adds it to the Ticker.
 * Note: if the Ticker actor is invalid or does not contain a Ticker Component, the actor will not spawn.
 */
USTRUCT(DisplayName="Spawn Actor to Ticker", Category="Motion Design", meta=(MotionDesignTask))
struct FAvaSceneStateSpawnerTickerTask : public FSceneStateSpawnActorTask
{
	GENERATED_BODY()

	using FInstanceDataType = FAvaSceneStateTickerTaskInstance;

protected:
	//~ Begin FSceneStateTask
#if WITH_EDITOR
	UE_API virtual const UScriptStruct* OnGetTaskInstanceType() const override;
#endif
	//~ End FSceneStateTask

	//~ Begin FSceneStateSpawnActorTask
	UE_API virtual bool ShouldSpawnActor(FStructView InTaskInstance, FText& OutErrorMessage) const override;
	UE_API virtual void OnActorSpawned(AActor* InActorChecked, FStructView InTaskInstance) const override;
	//~ End FSceneStateSpawnActorTask
};

#undef UE_API
