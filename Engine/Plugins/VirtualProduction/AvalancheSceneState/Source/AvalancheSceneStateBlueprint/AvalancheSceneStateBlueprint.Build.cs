// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheSceneStateBlueprint : ModuleRules
{
    public AvalancheSceneStateBlueprint(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
                "Engine",
                "PropertyBindingUtils",
                "SceneStateBlueprint",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "AvalancheSceneState",
                "CoreUObject",
                "SceneState",
            }
        );
    }
}