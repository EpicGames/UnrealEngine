// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSceneStateBlueprint.h"
#include "AvaSceneStateSchema.h"
#include "SceneStateVersion.h"

void UAvaSceneStateBlueprint::PostLoad()
{
	Super::PostLoad();

	// Previous versions to default to the motion design schema type
	if (GetLinkerCustomVersion(UE::SceneState::FVersion::Guid) < UE::SceneState::FVersion::Schema)
	{
		SetSceneStateSchema(UAvaSceneStateSchema::StaticClass());
	}
}
