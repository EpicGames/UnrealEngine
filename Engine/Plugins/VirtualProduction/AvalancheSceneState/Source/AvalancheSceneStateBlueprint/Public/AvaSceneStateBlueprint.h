// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneStateBlueprint.h"
#include "AvaSceneStateBlueprint.generated.h"

#define UE_API AVALANCHESCENESTATEBLUEPRINT_API

UCLASS(MinimalAPI, DisplayName="Motion Design Scene State Blueprint")
class UAvaSceneStateBlueprint : public USceneStateBlueprint
{
	GENERATED_BODY()

public:
	//~ Begin UObject
	UE_API virtual void PostLoad() override;
	//~ End UObject
};

#undef UE_API
