// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AvalancheSceneStateEditor : ModuleRules
{
    public AvalancheSceneStateEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "AvalancheEditorCore",
                "AvalancheSceneState",
                "AvalancheSceneStateBlueprint",
                "BlueprintGraph",
                "Core",
                "CoreUObject",
                "Engine",
                "KismetWidgets",
                "Projects",
                "PropertyEditor",
                "RemoteControlLogic",
                "SceneStateBlueprint",
                "SceneStateBlueprintEditor",
                "SceneStateGameplay",
                "Slate",
                "SlateCore",
                "StructUtilsEditor",
                "ToolMenus",
                "UnrealEd",
            }
        );
    }
}