// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSceneStateEditorModule.h"
#include "AvaEditorBuilder.h"
#include "AvaSceneStateBlueprint.h"
#include "AvaSceneStateEditorStyle.h"
#include "AvaSceneStateExtension.h"
#include "DetailsView/AvaSceneStateRCTaskDetails.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "RemoteControl/AvaSceneStateRCEventBehaviorNode.h"
#include "RemoteControl/AvaSceneStateRCTask.h"

IMPLEMENT_MODULE(FAvaSceneStateEditorModule, AvalancheSceneStateEditor)

void FAvaSceneStateEditorModule::StartupModule()
{
	UE::AvaSceneState::Editor::FEditorStyle::Get();

	OnEditorBuildHandle = FAvaEditorBuilder::OnEditorBuild.AddLambda(
		[](FAvaEditorBuilder& InBuilder)
		{
			InBuilder.AddExtension<FAvaSceneStateExtension>();	
		});

	RegisterCustomizations();
}

void FAvaSceneStateEditorModule::ShutdownModule()
{
	FAvaEditorBuilder::OnEditorBuild.Remove(OnEditorBuildHandle);
	OnEditorBuildHandle.Reset();

	UnregisterCustomizations();
}

void FAvaSceneStateEditorModule::RegisterCustomizations()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyEditorModule.RegisterCustomPropertyTypeLayout(CustomizedTypes.Add_GetRef(FAvaSceneStateRCTaskInstance::StaticStruct()->GetFName())
		, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAvaSceneStateRCTaskDetails::MakeInstance));
}

void FAvaSceneStateEditorModule::UnregisterCustomizations()
{
	if (FPropertyEditorModule* PropertyEditorModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
	{
		for (FName CustomizedType : CustomizedTypes)
		{
			PropertyEditorModule->UnregisterCustomPropertyTypeLayout(CustomizedType);	
		}
		CustomizedTypes.Reset();
	}
}
