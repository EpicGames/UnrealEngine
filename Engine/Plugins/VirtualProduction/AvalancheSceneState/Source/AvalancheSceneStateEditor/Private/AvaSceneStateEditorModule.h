// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Delegates/IDelegateInstance.h"
#include "Modules/ModuleInterface.h"
#include "UObject/NameTypes.h"

class FAvaSceneStateEditorModule : public IModuleInterface
{
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

	void RegisterCustomizations();
	void UnregisterCustomizations();

	FDelegateHandle OnEditorBuildHandle;

	TArray<FName> CustomizedTypes;
};
