// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSceneStateEditorStyle.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"

namespace UE::AvaSceneState::Editor
{

FEditorStyle::FEditorStyle()
	: FSlateStyleSet(TEXT("AvaSceneStateEditor"))
{
	ContentRootDir = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME)->GetContentDir();
	CoreContentRootDir = FPaths::EngineContentDir() / TEXT("Slate");

	// Task Icons
	Set("TaskIcon.AvaSceneStatePlaySequenceTask", new IMAGE_BRUSH_SVG("Icons/Tasks/PlaySequence", CoreStyleConstants::Icon16x16));
	Set("TaskIcon.AvaSceneStateRCTask", new IMAGE_BRUSH("Icons/Tasks/RemoteControl", CoreStyleConstants::Icon16x16));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FEditorStyle::~FEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

}
