// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

namespace UE::AvaSceneState::Editor
{

class FEditorStyle : public FSlateStyleSet
{
public:
	static FEditorStyle& Get()
	{
		static FEditorStyle Instance;
		return Instance;
	}

	FEditorStyle();
	virtual ~FEditorStyle() override;
};

} // UE::AvaSceneState::Editor
