// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSceneStateExtension.h"
#include "AvaSceneStateActor.h"
#include "AvaSceneStateBlueprint.h"
#include "AvaSceneStateSchema.h"
#include "BlueprintActionDatabase.h"
#include "Editor.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorModeManager.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "Misc/MessageDialog.h"
#include "SceneStateActor.h"
#include "SceneStateBlueprintFactory.h"
#include "SceneStateComponent.h"
#include "SceneStateComponentPlayer.h"
#include "SceneStateObject.h"
#include "ScopedTransaction.h"
#include "Selection.h"
#include "Styling/SlateIconFinder.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "ToolMenu.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"
#include "UnrealEdGlobals.h"

#define LOCTEXT_NAMESPACE "AvaSceneStateExtension"

void FAvaSceneStateExtension::ExtendToolbarMenu(UToolMenu& InMenu)
{
	FToolMenuSection& Section = InMenu.FindOrAddSection(DefaultSectionName);

	FSlateIcon SceneStateIcon = FSlateIconFinder::FindCustomIconForClass(UAvaSceneStateBlueprint::StaticClass(), TEXT("ClassThumbnail"));

	FToolMenuEntry& Entry = Section.AddEntry(FToolMenuEntry::InitToolBarButton(TEXT("SceneStateButton")
		, FExecuteAction::CreateSP(this, &FAvaSceneStateExtension::OpenSceneStateBlueprintEditor)
		, LOCTEXT("SceneStateLabel", "Scene State")
		, LOCTEXT("SceneStateTooltip", "Opens the Scene State Editor for the given Motion Design Scene")
		, SceneStateIcon));

	Entry.StyleNameOverride = TEXT("CalloutToolbar");

	Section.AddEntry(FToolMenuEntry::InitComboButton(TEXT("SceneStateComboButton")
		, FUIAction()
		, FNewToolMenuDelegate::CreateSP(this, &FAvaSceneStateExtension::GenerateSceneStateOptions)
		, LOCTEXT("SceneStateOptionsLabel", "Scene State Options")
		, LOCTEXT("SceneStateOptionsTooltip", "Scene State Options")
		, SceneStateIcon
		, /*bSimpleComboBox*/true));
}

void FAvaSceneStateExtension::Cleanup()
{
	UWorld* const World = GetWorld();
	if (!World)
	{
		return;
	}

	FBlueprintActionDatabase* BlueprintActionDatabase = FBlueprintActionDatabase::TryGet();
	if (!BlueprintActionDatabase)
	{
		return;
	}

	for (AAvaSceneStateActor* SceneStateActor : TActorRange<AAvaSceneStateActor>(World))
	{
		if (SceneStateActor->SceneStateBlueprint)
		{
			BlueprintActionDatabase->ClearAssetActions(SceneStateActor->SceneStateBlueprint.Get());
		}
	}
}

void FAvaSceneStateExtension::GenerateSceneStateOptions(UToolMenu* InMenu)
{
	FToolMenuSection& GeneralSection = InMenu->FindOrAddSection(TEXT("GeneralSection"), LOCTEXT("GeneralSectionLabel", "General"));

	GeneralSection.AddMenuEntry(TEXT("DeleteSceneState")
		, LOCTEXT("DeleteSceneStateLabel", "Delete Scene State")
		, LOCTEXT("DeleteSceneStateTooltip", "Deletes the existing embedded Scene State")
		, TAttribute<FSlateIcon>()
		, FUIAction(FExecuteAction::CreateSP(this, &FAvaSceneStateExtension::DeleteSceneStateActor)
			, FCanExecuteAction::CreateSP(this, &FAvaSceneStateExtension::CanDeleteSceneStateActor))
		, EUserInterfaceActionType::Button);
}

AAvaSceneStateActor* FAvaSceneStateExtension::FindSceneStateActor() const
{
	if (UWorld* const World = GetWorld())
	{
		if (TActorIterator<AAvaSceneStateActor>It(World); It)
		{
			return *It;
		}
	}
	return nullptr;
}

AAvaSceneStateActor* FAvaSceneStateExtension::FindOrSpawnSceneStateActor() const
{
	UWorld* const World = GetWorld();
	if (!World)
	{
		return nullptr;
	}

	if (TActorIterator<AAvaSceneStateActor>It(World); It)
	{
		return *It;
	}

	return World->SpawnActor<AAvaSceneStateActor>();
}

USceneStateBlueprint* FAvaSceneStateExtension::CreateSceneStateBlueprint(AAvaSceneStateActor* InSceneStateActor) const
{
	check(InSceneStateActor);

	USceneStateBlueprintFactory* Factory = NewObject<USceneStateBlueprintFactory>(GetTransientPackage());
	check(Factory);
	Factory->SchemaClass = UAvaSceneStateSchema::StaticClass();

	USceneStateBlueprint* NewSceneStateBlueprint = CastChecked<USceneStateBlueprint>(Factory->FactoryCreateNew(UAvaSceneStateBlueprint::StaticClass()
		, InSceneStateActor
		, TEXT("SceneStateBlueprint")
		, RF_Transactional
		, nullptr
		, GWarn));

	// Clear Standalone flags as this Blueprint will be under the Scene State Actor
	NewSceneStateBlueprint->ClearFlags(RF_Standalone);

	InSceneStateActor->SetSceneStateBlueprint(NewSceneStateBlueprint);
	InSceneStateActor->UpdateSceneStateClass();

	return NewSceneStateBlueprint;
}

bool FAvaSceneStateExtension::CanDeleteSceneStateActor() const
{
	return !!FindSceneStateActor();
}

void FAvaSceneStateExtension::DeleteSceneStateActor()
{
	const EAppReturnType::Type Response = FMessageDialog::Open(EAppMsgType::YesNo
		, LOCTEXT("DeleteSceneStateMessage", "Do you wish to delete this Scene State Actor?\nThe embedded scene state blueprint will be destroyed with it.")
		, LOCTEXT("DeleteSceneStateTitle", "Delete Scene State"));

	if (Response != EAppReturnType::Yes)
	{
		return;
	}

	AAvaSceneStateActor* SceneStateActor = FindSceneStateActor();
	if (!SceneStateActor)
	{
		return;
	}

	if (SceneStateActor->SceneStateBlueprint)
	{
		UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
		check(AssetEditorSubsystem);
		AssetEditorSubsystem->CloseAllEditorsForAsset(SceneStateActor->SceneStateBlueprint);
	}

	SceneStateActor->CleanupSceneState();

	if (const FEditorModeTools* EditorModeTools = GetEditorModeTools())
	{
		if (USelection* SelectedComponents = EditorModeTools->GetSelectedComponents())
		{
			if (USceneStateComponent* SceneStateComponent = SceneStateActor->GetSceneStateComponent())
			{
				SelectedComponents->Deselect(SceneStateComponent);	
			}
		}
		if (USelection* SelectedActors = EditorModeTools->GetSelectedActors())
		{
			SelectedActors->Deselect(SceneStateActor);
		}
	}

	SceneStateActor->Destroy(/*bNetForce*/false, /*bShouldModifyLevel*/false);
}

void FAvaSceneStateExtension::OpenSceneStateBlueprintEditor()
{
	AAvaSceneStateActor* SceneStateActor = FindOrSpawnSceneStateActor();
	if (!SceneStateActor)
	{
		return;
	}

	if (!SceneStateActor->SceneStateBlueprint)
	{
		CreateSceneStateBlueprint(SceneStateActor);
	}
	check(SceneStateActor->SceneStateBlueprint);

	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	check(AssetEditorSubsystem);
	AssetEditorSubsystem->OpenEditorForAsset(SceneStateActor->SceneStateBlueprint);
}

#undef LOCTEXT_NAMESPACE
