// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAvaEditorExtension.h"

class AAvaSceneStateActor;
class USceneStateBlueprint;

class FAvaSceneStateExtension : public FAvaEditorExtension
{
public:
	UE_AVA_INHERITS(FAvaSceneStateExtension, FAvaEditorExtension);

	//~ Begin IAvaEditorExtension
	virtual void ExtendToolbarMenu(UToolMenu& InMenu) override;
	virtual void Cleanup() override;
	//~ End IAvaEditorExtension

private:
	void GenerateSceneStateOptions(UToolMenu* InMenu);

	/** Finds an existing scene state actor in the current world */
	AAvaSceneStateActor* FindSceneStateActor() const;

	/** Finds or spawns a scene state actor in the current world */
	AAvaSceneStateActor* FindOrSpawnSceneStateActor() const;

	/** Creates a new scene state blueprint and assigns it to the given scene state actor */
	USceneStateBlueprint* CreateSceneStateBlueprint(AAvaSceneStateActor* InSceneStateActor) const;

	/** Returns whether there's an existing scene state actor that can be destroyed */
	bool CanDeleteSceneStateActor() const;

	/** Destroys the existing scene state actor in the current world */
	void DeleteSceneStateActor();

	/** Opens the existing scene state blueprint in the current world*/
	void OpenSceneStateBlueprintEditor();
};
