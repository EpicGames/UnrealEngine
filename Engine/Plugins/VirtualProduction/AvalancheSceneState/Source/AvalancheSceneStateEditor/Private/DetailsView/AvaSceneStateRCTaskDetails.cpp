// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSceneStateRCTaskDetails.h"
#include "AvaSceneStateRCValuesDetails.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyBagDetails.h"
#include "PropertyHandle.h"
#include "RemoteControl/AvaSceneStateRCTask.h"
#include "SceneStateBlueprintEditorUtils.h"

void FAvaSceneStateRCTaskDetails::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle
	, FDetailWidgetRow& InHeaderRow
	, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
}

void FAvaSceneStateRCTaskDetails::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle
	, IDetailChildrenBuilder& InChildBuilder
	, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	const TSharedRef<IPropertyHandle> ControllerValuesIdHandle = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaSceneStateRCTaskInstance, ControllerValuesId)).ToSharedRef();
	ControllerValuesIdHandle->MarkHiddenByCustomization();

	// If the id could not be gotten, continue displaying the customization as normal.
	// Not having an id only means that these rows will not be bindable
	FGuid ControllerValuesId;
	UE::SceneState::Editor::GetGuid(ControllerValuesIdHandle, ControllerValuesId);

	InChildBuilder.AddCustomBuilder(MakeShared<FAvaSceneStateRCValuesDetails>(InPropertyHandle, ControllerValuesId, InCustomizationUtils.GetPropertyUtilities()));
}
