// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSceneStateRCUtils.h"
#include "EdGraph/EdGraphPin.h"
#include "Misc/EnumerateRange.h"
#include "PropertyBagDetails.h"
#include "PropertyHandle.h"
#include "RemoteControl/AvaSceneStateRCTask.h"
#include "RemoteControlLogicConfig.h"

namespace UE::AvaSceneStateRCUtils
{

void ForEachControllerSupportedType(TFunctionRef<void(const FEdGraphPinType&)> InFunctor)
{
	const URemoteControlLogicConfig* RCLogicConfig = GetDefault<URemoteControlLogicConfig>();
	if (!ensure(RCLogicConfig))
	{
		return;
	}

	for (EPropertyBagPropertyType ControllerType : RCLogicConfig->SupportedControllerTypes)
	{
		// Skip structs/objects. These are handled in a separate pass with SupportedControllerStructTypes and SupportedControllerObjectClassPaths
		if (ControllerType == EPropertyBagPropertyType::Struct || ControllerType == EPropertyBagPropertyType::Object)
		{
			continue;
		}

		InFunctor(UE::StructUtils::GetPropertyDescAsPin(FPropertyBagPropertyDesc(NAME_None, ControllerType)));
	}

	constexpr const TCHAR* CoreStructTypePathPrefix = TEXT("/Script/CoreUObject.");
	for (FName ControllerStructType : RCLogicConfig->SupportedControllerStructTypes)
	{
		if (UStruct* Struct = FindObject<UScriptStruct>(nullptr, *(CoreStructTypePathPrefix + ControllerStructType.ToString())))
		{
			InFunctor(UE::StructUtils::GetPropertyDescAsPin(FPropertyBagPropertyDesc(NAME_None, EPropertyBagPropertyType::Struct, Struct)));
		}
	}

	for (FName ControllerObjectType : RCLogicConfig->SupportedControllerObjectClassPaths)
	{
		if (UObject* Object = FSoftClassPath(ControllerObjectType.ToString()).TryLoad())
		{
			InFunctor(UE::StructUtils::GetPropertyDescAsPin(FPropertyBagPropertyDesc(NAME_None, EPropertyBagPropertyType::Object, Object)));
		}
	}
}

FEdGraphPinType GetPinInfo(const TSharedRef<IPropertyHandle>& InPropertyHandle)
{
	if (!InPropertyHandle->IsValidHandle())
	{
		return FEdGraphPinType();
	}
	if (const FProperty* ValueProperty = InPropertyHandle->GetProperty())
	{
		return UE::StructUtils::GetPropertyDescAsPin(FPropertyBagPropertyDesc(NAME_None, ValueProperty));
	}
	return FEdGraphPinType();
}

FPropertyBagPropertyDesc MakeControllerPropertyDesc()
{
	FPropertyBagPropertyDesc PropertyDesc;
	PropertyDesc.ID = FGuid::NewGuid();
	PropertyDesc.Name = FName(TEXT("Property_") + PropertyDesc.ID.ToString());

	// todo: a controller could have more info on the type to set
	PropertyDesc.ValueType = EPropertyBagPropertyType::String;
	return PropertyDesc;
}

bool SyncPropertyDescs(TArray<FPropertyBagPropertyDesc>& InOutPropertyDescs, TArrayView<FAvaSceneStateRCControllerMapping> InMappings)
{
	// Remove all the property descs that are no longer in mapping
	InOutPropertyDescs.RemoveAll(
		[&InMappings](const FPropertyBagPropertyDesc& InPropertyDesc)
		{
			return !InMappings.FindByKey(InPropertyDesc.ID);
		});

	InOutPropertyDescs.Reserve(InMappings.Num());

	// property descs cannot have more elements than mappings.
	// otherwise it means there are mappings that have repeated property desc id
	if (!ensure(InOutPropertyDescs.Num() <= InMappings.Num()))
	{
		return false;
	}

	// Create new property descs to mappings that do not have a matching desc
	for (TEnumerateRef<FAvaSceneStateRCControllerMapping> Mapping : EnumerateRange(InMappings))
	{
		const int32 MappingIndex = Mapping.GetIndex();

		// No property desc available or mapping is new (and invalid), can add directly
		if (!InOutPropertyDescs.IsValidIndex(MappingIndex) || !Mapping->SourcePropertyId.IsValid())
		{
			// mapping should not have a valid property id here
			if (!ensure(!Mapping->SourcePropertyId.IsValid()))
			{
				return false;
			}

			// create a new desc at the mapping index and link the mapping to it
			FPropertyBagPropertyDesc& Desc = InOutPropertyDescs.Insert_GetRef(UE::AvaSceneStateRCUtils::MakeControllerPropertyDesc(), MappingIndex);
			Mapping->SourcePropertyId = Desc.ID;
			continue;
		}

		// mapping should have a valid property at this point (new mapping entries added with invalid ids should've been handled above)
		if (!ensure(Mapping->SourcePropertyId.IsValid()))
		{
			return false;
		}

		int32 PropertyDescIndex = INDEX_NONE;

		// Find the property desc that matches the valid mapping source id
		// start from the mapping index as anything before that should already be fixed (i.e. the ids for the elements at each array match)
		for (int32 Index = MappingIndex; Index < InOutPropertyDescs.Num(); ++Index)
		{
			if (InOutPropertyDescs[Index].ID == Mapping->SourcePropertyId)
			{
				PropertyDescIndex = Index;
				break;
			}
		}

		// Property index must've been found, as all the property desc entries that didn't have a matching mapping were removed ahead of time
		if (!ensure(PropertyDescIndex != INDEX_NONE))
		{
			return false;
		}

		// the index matched the current mapping, nothing to do
		if (PropertyDescIndex == MappingIndex)
		{
			continue;
		}

		// move the property desc to the mapping index
		FPropertyBagPropertyDesc PropertyDesc = MoveTemp(InOutPropertyDescs[PropertyDescIndex]);
		InOutPropertyDescs.RemoveAt(PropertyDescIndex);
		InOutPropertyDescs.Insert(MoveTemp(PropertyDesc), MappingIndex);
	}

	// validation pass: mapping count must match property desc count now
	if (!ensure(InOutPropertyDescs.Num() == InMappings.Num()))
	{
		return false;
	}

	// validation pass: all ids must match for each index
	for (int32 Index = 0; Index < InMappings.Num(); ++Index)
	{
		if (!ensure(InMappings[Index].SourcePropertyId == InOutPropertyDescs[Index].ID))
		{
			return false;
		}
	}

	return true;
}

} // UE::AvaSceneStateRCUtils
