// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Templates/FunctionFwd.h"
#include "Templates/SharedPointerFwd.h"

class IPropertyHandle;
class UPropertyBag;
struct FAvaSceneStateRCControllerMapping;
struct FEdGraphPinType;
struct FPropertyBagPropertyDesc;

namespace UE::AvaSceneStateRCUtils
{
	/** Calls the given functor for every supported type for Remote Control controllers */
	void ForEachControllerSupportedType(TFunctionRef<void(const FEdGraphPinType&)> InFunctor);

	/** Gets the pin type for the property under the given property handle */
	FEdGraphPinType GetPinInfo(const TSharedRef<IPropertyHandle>& InPropertyHandle);

	/** Creates a new generic property desc */
	FPropertyBagPropertyDesc MakeControllerPropertyDesc();

	/**
	 * Returns a copy of property descs (some that could already exist in the property bag) that matches the structure in the given mappings
	 * @param InOutPropertyDescs the array of property descs to sync 
	 * @param InMappings the mappings to match against
	 */
	bool SyncPropertyDescs(TArray<FPropertyBagPropertyDesc>& InOutPropertyDescs, TArrayView<FAvaSceneStateRCControllerMapping> InMappings);
}
