// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaSceneStateRCValuesDetails.h"
#include "AvaSceneStateRCUtils.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "IPropertyUtilities.h"
#include "IStructureDataProvider.h"
#include "Misc/EnumerateRange.h"
#include "RemoteControl/AvaSceneStateRCTask.h"
#include "SPinTypeSelector.h"
#include "SceneStateBlueprintEditorUtils.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "AvaSceneStateRCValuesDetails"

FAvaSceneStateRCValuesDetails::FAvaSceneStateRCValuesDetails(const TSharedRef<IPropertyHandle>& InStructHandle, const FGuid& InValuesId, TSharedPtr<IPropertyUtilities> InPropUtils)
	: StructHandle(InStructHandle)
	, ValuesHandle(InStructHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaSceneStateRCTaskInstance, ControllerValues)).ToSharedRef())
	, MappingsHandle(InStructHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAvaSceneStateRCTaskInstance, ControllerMappings)).ToSharedRef())
	, PropertyUtilities(InPropUtils)
	, ValuesId(InValuesId)
{
	ValuesHandle->MarkHiddenByCustomization();
	MappingsHandle->MarkHiddenByCustomization();

	UE::SceneState::Editor::AssignBindingId(MappingsHandle, UE::SceneState::Editor::FindTaskId(InStructHandle));
	UE::SceneState::Editor::AssignBindingId(ValuesHandle, ValuesId);
}

void FAvaSceneStateRCValuesDetails::Initialize()
{
	bInitialized = true;

	TSharedRef<IPropertyHandleArray> ControllerMappingsArray = MappingsHandle->AsArray().ToSharedRef();

	// NumElementsChanged gets called on RebuildChildren at a time out of a transaction
	ControllerMappingsArray->SetOnNumElementsChanged(FSimpleDelegate::CreateSP(this, &FAvaSceneStateRCValuesDetails::OnControllerMappingsNumChanged));

	// PropertyValueChanged gets called while still on a transaction
	// Sync controller values whenever the mappings change
	MappingsHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FAvaSceneStateRCValuesDetails::SyncControllerValues));
}

void FAvaSceneStateRCValuesDetails::SyncControllerValues()
{
	TArray<FAvaSceneStateRCControllerMapping>& Mappings = GetControllerMappings();

	FInstancedPropertyBag& InstancedPropertyBag =  GetInstancedPropertyBag();

	// Copy current layout of property bag struct to mutate into latest layout
	TArray<FPropertyBagPropertyDesc> PropertyDescs;
	if (const UPropertyBag* PropertyBag = InstancedPropertyBag.GetPropertyBagStruct())
	{
		PropertyDescs = PropertyBag->GetPropertyDescs();
	}

	// Sync PropertyDescs to match the mapping entries
	if (!UE::AvaSceneStateRCUtils::SyncPropertyDescs(/*in/out*/PropertyDescs, Mappings))
	{
		return;
	}

	ValuesHandle->NotifyPreChange();

	InstancedPropertyBag.MigrateToNewBagStruct(UPropertyBag::GetOrCreateFromDescs(PropertyDescs));

	ValuesHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	ValuesHandle->NotifyFinishedChangingProperties();

	if (PropertyUtilities.IsValid())
	{
		PropertyUtilities->RequestForceRefresh();
	}

	OnRebuildChildren.ExecuteIfBound();
}

void FAvaSceneStateRCValuesDetails::OnControllerMappingsNumChanged()
{
	OnRebuildChildren.ExecuteIfBound();
}

void FAvaSceneStateRCValuesDetails::ConfigureMappingRow(const TSharedRef<IPropertyHandle>& InMappingHandle, IDetailPropertyRow& InChildRow)
{
	FDetailWidgetRow& WidgetRow = InChildRow.CustomWidget(/*bShowChildren*/true);
	{
		TSharedPtr<SWidget> NameWidget;
		TSharedPtr<SWidget> ValueWidget;
		InChildRow.GetDefaultWidgets(NameWidget, ValueWidget, WidgetRow);
	}

	WidgetRow
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				WidgetRow.ValueWidget.Widget
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(4, 1, 0, 1)
			[
				InMappingHandle->CreateDefaultPropertyButtonWidgets()
			]
		];
}

void FAvaSceneStateRCValuesDetails::ConfigureValueRow(IDetailPropertyRow& InChildRow)
{
	TSharedRef<IPropertyHandle> ValueHandle = InChildRow.GetPropertyHandle().ToSharedRef();

	FDetailWidgetRow& WidgetRow = InChildRow.CustomWidget(/*bShowChildren*/true);
	{
		TSharedPtr<SWidget> NameWidget;
		TSharedPtr<SWidget> ValueWidget;
		InChildRow.GetDefaultWidgets(NameWidget, ValueWidget, WidgetRow);
	}

	UE::SceneState::Editor::AssignBindingId(ValueHandle, ValuesId);

	WidgetRow
		.NameContent()
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
			.WidthOverride(90)
			.Padding(0, 0, 4, 0)
			[
				SNew(SPinTypeSelector, FGetPinTypeTree::CreateSP(this, &FAvaSceneStateRCValuesDetails::GetControllerSupportedTypes))
				.TargetPinType_Lambda([ValueHandle]()->FEdGraphPinType
					{
						return UE::AvaSceneStateRCUtils::GetPinInfo(ValueHandle);
					})
				.SelectorType(SPinTypeSelector::ESelectorType::Partial)
				.OnPinTypeChanged(this, &FAvaSceneStateRCValuesDetails::OnPinInfoChanged, ValueHandle)
				.Schema(GetDefault<UPropertyBagSchema>())
				.bAllowArrays(false)
				.TypeTreeFilter(ETypeTreeFilter::None)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];
}

void FAvaSceneStateRCValuesDetails::OnPinInfoChanged(const FEdGraphPinType& InPinType, TSharedRef<IPropertyHandle> InValueHandle)
{
	if (!InValueHandle->IsValidHandle())
	{
		return;
	}

	const FProperty* ValueProperty = InValueHandle->GetProperty();
	if (!ValueProperty)
	{
		return;
	}

	const UPropertyBag* CurrentPropertyBag = GetInstancedPropertyBag().GetPropertyBagStruct();
	if (!CurrentPropertyBag)
	{
		return;
	}

	TArray<FPropertyBagPropertyDesc> PropertyDescs(CurrentPropertyBag->GetPropertyDescs());

	FPropertyBagPropertyDesc* const PropertyDesc = PropertyDescs.FindByPredicate(
		[ValueProperty](const FPropertyBagPropertyDesc& InDesc)
		{
			return InDesc.CachedProperty == ValueProperty;
		});

	if (!PropertyDesc)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("OnPropertyTypeChanged", "Change Property Type"));
	ValuesHandle->NotifyPreChange();

	UE::StructUtils::SetPropertyDescFromPin(*PropertyDesc, InPinType);
	GetInstancedPropertyBag().MigrateToNewBagStruct(UPropertyBag::GetOrCreateFromDescs(PropertyDescs));

	ValuesHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
	ValuesHandle->NotifyFinishedChangingProperties();
}

void FAvaSceneStateRCValuesDetails::GetControllerSupportedTypes(TArray<FPinTypeTreeItem>& OutTreeItems, ETypeTreeFilter InTreeFilter)
{
	OutTreeItems.Reset();

	const UPropertyBagSchema* PropertyBagSchema = GetDefault<UPropertyBagSchema>();
	check(PropertyBagSchema);

	UE::AvaSceneStateRCUtils::ForEachControllerSupportedType(
		[&OutTreeItems, PropertyBagSchema](const FEdGraphPinType& InPinType)
		{
			FPinTypeTreeItem TreeItem;
			if (UObject* SubCategoryObject = InPinType.PinSubCategoryObject.Get())
			{
				TreeItem = MakeShared<UEdGraphSchema_K2::FPinTypeTreeInfo>(InPinType.PinCategory
					, SubCategoryObject
					, FText::GetEmpty());
			}
			else
			{
				const FText CategoryText = UPropertyBagSchema::GetCategoryText(InPinType.PinCategory);
				TreeItem = MakeShared<UEdGraphSchema_K2::FPinTypeTreeInfo>(CategoryText, InPinType.PinCategory, PropertyBagSchema, FText::GetEmpty());
			}

			TreeItem->SetPinSubTypeCategory(InPinType.PinSubCategory);
			OutTreeItems.Add(MoveTemp(TreeItem));
		});

	check(PropertyBagSchema);
}

FName FAvaSceneStateRCValuesDetails::GetName() const
{
	return TEXT("FAvaSceneStateRCValuesDetails");
}

void FAvaSceneStateRCValuesDetails::GenerateHeaderRowContent(FDetailWidgetRow& InNodeRow)
{
	if (!bInitialized)
	{
		Initialize();
	}

	InNodeRow
		.NameContent()
		[
			MappingsHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.VAlign(VAlign_Center)
		[
			MappingsHandle->CreatePropertyValueWidget(/*bDisplayDefaultPropertyButtons*/true)
		]
		.ShouldAutoExpand(true);
}

void FAvaSceneStateRCValuesDetails::GenerateChildContent(IDetailChildrenBuilder& InChildrenBuilder)
{
	// Get the FInstancedStruct property  under the FInstancedPropertyBag
	TSharedRef<IPropertyHandle> ControllerValuesStructHandle = ValuesHandle->GetChildHandle(TEXT("Value")).ToSharedRef();
	TSharedRef<IStructureDataProvider> ControllerValuesDataProvider = UE::SceneState::Editor::CreateInstancedStructDataProvider(ControllerValuesStructHandle);

	ControllerValuesStructHandle->RemoveChildren();

	TSharedRef<IPropertyHandleArray> ControllerMappingsArray = MappingsHandle->AsArray().ToSharedRef();

	uint32 ControllerMappingCount;
	ControllerMappingsArray->GetNumElements(ControllerMappingCount);

	const TArray<TSharedPtr<IPropertyHandle>> ControllerValueProperties = ControllerValuesStructHandle->AddChildStructure(ControllerValuesDataProvider);

	const uint32 Count = FMath::Min(static_cast<uint32>(ControllerValueProperties.Num()), ControllerMappingCount);

	const FName TargetControllerName = GET_MEMBER_NAME_CHECKED(FAvaSceneStateRCControllerMapping, TargetController);

	// Add two rows: the mapping controller id and the value of it
	for (uint32 Index = 0; Index < Count; ++Index)
	{
		TSharedRef<IPropertyHandle> MappingHandle = ControllerMappingsArray->GetElement(Index);
		ConfigureMappingRow(MappingHandle, InChildrenBuilder.AddProperty(MappingHandle->GetChildHandle(TargetControllerName).ToSharedRef()));
		ConfigureValueRow(InChildrenBuilder.AddProperty(ControllerValueProperties[Index].ToSharedRef()));
	}
}

void FAvaSceneStateRCValuesDetails::SetOnRebuildChildren(FSimpleDelegate InOnRebuildChildren)
{
	OnRebuildChildren = InOnRebuildChildren;
}

TArray<FAvaSceneStateRCControllerMapping>& FAvaSceneStateRCValuesDetails::GetControllerMappings()
{
	TArray<void*> MappingsRawData;
	MappingsHandle->AccessRawData(MappingsRawData);
	check(MappingsRawData.Num() == 1);
	return *static_cast<TArray<FAvaSceneStateRCControllerMapping>*>(MappingsRawData[0]);
}

FInstancedPropertyBag& FAvaSceneStateRCValuesDetails::GetInstancedPropertyBag()
{
	TArray<void*> ValuesRawData;
	ValuesHandle->AccessRawData(ValuesRawData);
	check(ValuesRawData.Num() == 1);
	return *static_cast<FInstancedPropertyBag*>(ValuesRawData[0]);
}

#undef LOCTEXT_NAMESPACE
