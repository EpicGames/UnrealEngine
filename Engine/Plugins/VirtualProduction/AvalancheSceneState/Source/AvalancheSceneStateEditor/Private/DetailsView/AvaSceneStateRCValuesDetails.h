// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"
#include "PropertyBagDetails.h"
#include "PropertyCustomizationHelpers.h"
#include "SPinTypeSelector.h"
#include "Templates/SharedPointer.h"

struct FAvaSceneStateRCControllerMapping;

/** Builder to group the controller mapping entries to the instanced property bag value properties */
class FAvaSceneStateRCValuesDetails : public IDetailCustomNodeBuilder, public TSharedFromThis<FAvaSceneStateRCValuesDetails>
{
public:
	explicit FAvaSceneStateRCValuesDetails(const TSharedRef<IPropertyHandle>& InStructHandle, const FGuid& InValuesId, TSharedPtr<IPropertyUtilities> InPropUtils);

	/** Called once per instance to bind to delegates */
	void Initialize();

	/**
	 * Called when mapping has changed, while still during a transaction.
	 * Syncs the value instanced property bag ensuring there's a valid matching value to every mapping entry.
	 */
	void SyncControllerValues();

	/** Called on rebuild time when the mapping num has changed */
	void OnControllerMappingsNumChanged();

	/** Customizes the controller mapping row */
	void ConfigureMappingRow(const TSharedRef<IPropertyHandle>& InMappingHandle, IDetailPropertyRow& InChildRow);

	/** Customizes the value row  */
	void ConfigureValueRow(IDetailPropertyRow& InChildRow);

	/** Called when the type of the value property has changed */
	void OnPinInfoChanged(const FEdGraphPinType& InPinType, TSharedRef<IPropertyHandle> InValueHandle);

	/** Gathers all the supported types for a remote control preset controller */
	void GetControllerSupportedTypes(TArray<FPinTypeTreeItem>& OutTreeItems, ETypeTreeFilter InTreeFilter);

	//~ Begin IDetailCustomNodeBuilder
	virtual FName GetName() const override;
	virtual bool InitiallyCollapsed() const override { return false; }
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& InNodeRow) override;
	virtual void GenerateChildContent(IDetailChildrenBuilder& InChildrenBuilder) override;
	virtual void SetOnRebuildChildren(FSimpleDelegate InOnRebuildChildren ) override;
	//~ End IDetailCustomNodeBuilder

private:
	/** Gets the controller mappings. Expects only one instance to be available */
	TArray<FAvaSceneStateRCControllerMapping>& GetControllerMappings();

	/** Gets the instanced property bag from ValuesHandle. Expects only one instance to be available */
	FInstancedPropertyBag& GetInstancedPropertyBag();

	/** Handle to the FAvaSceneStateRCTaskInstance holding the values and mappings properties */
	TSharedRef<IPropertyHandle> StructHandle;

	/** Handle to the controller values instanced property bag in FAvaSceneStateRCTaskInstance */
	TSharedRef<IPropertyHandle> ValuesHandle;

	/** Handle to the controller mappings array in FAvaSceneStateRCTaskInstance */
	TSharedRef<IPropertyHandle> MappingsHandle;

	/** Property utilities to use for refreshing the details view */
	TSharedPtr<IPropertyUtilities> PropertyUtilities;

	/** Binding id to the values Instanced property bag */
	FGuid ValuesId;

	/** Delegate called when needing to rebuild the children items */
	FSimpleDelegate OnRebuildChildren;

	/** Flag to ensure Initialize() only gets called once */
	bool bInitialized = false;
};
